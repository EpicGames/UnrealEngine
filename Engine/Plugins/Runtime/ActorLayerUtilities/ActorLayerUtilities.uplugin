{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Actor Layer Utilities",
	"Description": "Utilites for interacting with actor layers from blueprints",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "ActorLayerUtilities",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ActorLayerUtilitiesEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"IsExperimentalVersion": false
}