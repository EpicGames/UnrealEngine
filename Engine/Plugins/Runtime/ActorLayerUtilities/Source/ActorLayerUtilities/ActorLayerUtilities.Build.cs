// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ActorLayerUtilities : ModuleRules
{
	public ActorLayerUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Engine"
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject"
			}
		);
	}
}
