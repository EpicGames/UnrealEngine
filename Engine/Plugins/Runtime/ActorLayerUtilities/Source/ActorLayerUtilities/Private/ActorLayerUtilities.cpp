// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorLayerUtilities.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorLayerUtilities)

IMPLEMENT_MODULE(FDefaultModuleImpl, ActorLayerUtilities)

TArray<AActor*> ULayersBlueprintLibrary::GetActors(UObject* WorldContextObject, const FActorLayer& ActorLayer)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!World)
	{
		return TArray<AActor*>();
	}

	TArray<AActor*> AllActors;

	// Iterate over all actors, looking for actors in the specified layers.
	for (const TWeakObjectPtr<AActor> WeakActor : FActorRange(World))
	{
		AActor* Actor = WeakActor.Get();
		if (Actor && Actor->Layers.Contains(ActorLayer.Name))
		{
			AllActors.Add(Actor);
		}
	}

	return AllActors;
}

void ULayersBlueprintLibrary::AddActorToLayer(AActor* InActor, const FActorLayer& Layer)
{
	if (InActor && (Layer.Name != NAME_None))
	{
		InActor->Layers.AddUnique(Layer.Name);
	}
}

void ULayersBlueprintLibrary::RemoveActorFromLayer(AActor* InActor, const FActorLayer& Layer)
{
	if (InActor && (Layer.Name != NAME_None))
	{
		InActor->Layers.Remove(Layer.Name);
	}
}

