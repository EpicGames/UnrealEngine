// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "ActorLayerUtilities.generated.h"

USTRUCT(BlueprintType)
struct FActorLayer
{
	GENERATED_BODY()

	/** The name of this layer */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Layer)
	FName Name;
};


/**
 * Function library containing methods for interacting with editor layers
 */
UCLASS()
class ULayersBlueprintLibrary : public UBlueprintFunctionLibrary
{
public:

	GENERATED_BODY()

	/**
	 * Get all the actors in this layer
	 */
	UFUNCTION(BlueprintCallable, Category=Layer, meta=(WorldContext=WorldContextObject))
	static TArray<AActor*> GetActors(UObject* WorldContextObject, const FActorLayer& ActorLayer);

	/** 
	 * Adds the actor to the specified layer
	 */
	UFUNCTION(BlueprintCallable, Category = Layer)
	static void AddActorToLayer(AActor* InActor, const FActorLayer& Layer);

	/**
	 * Removes the actor from the specified layer
	 */
	UFUNCTION(BlueprintCallable, Category = Layer)
	static void RemoveActorFromLayer(AActor* InActor, const FActorLayer& Layer);


};
