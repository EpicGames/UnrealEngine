// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ActorLayerUtilitiesEditor : ModuleRules
{
	public ActorLayerUtilitiesEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ActorLayerUtilities",
				"Core",
				"CoreUObject",
				"EditorWidgets",
				"Engine",
				"Layers",
				"LevelEditor",
				"PropertyEditor",
				"SlateCore",
				"Slate",
				"UnrealEd",
				"EditorStyle",
			}
		);
	}
}
