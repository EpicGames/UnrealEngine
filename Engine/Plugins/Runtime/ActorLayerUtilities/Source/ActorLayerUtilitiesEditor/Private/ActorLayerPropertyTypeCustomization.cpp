// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorLayerPropertyTypeCustomization.h"
#include "Algo/Accumulate.h"
#include "Modules/ModuleManager.h"
#include "DetailWidgetRow.h"
#include "LayersDragDropOp.h"
#include "Layers/Layer.h"
#include "Layers/LayersSubsystem.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SDropTarget.h"
#include "EditorFontGlyphs.h"
#include "LevelEditor.h"


#define LOCTEXT_NAMESPACE "ActorLayerPropertyTypeCustomization"

void FActorLayerPropertyTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyHandle = StructPropertyHandle->GetChildHandle("Name");

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(TOptional<float>())
	.MaxDesiredWidth(TOptional<float>())
	[
		SNew(SDropTarget)
		.OnDropped(this, &FActorLayerPropertyTypeCustomization::OnDrop)
		.OnAllowDrop(this, &FActorLayerPropertyTypeCustomization::OnVerifyDrag)
		.OnIsRecognized(this, &FActorLayerPropertyTypeCustomization::OnVerifyDrag)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(TEXT("Layer.Icon16x")))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(3.0f, 0.0f)
			.FillWidth(1.0f)
			[
				SNew(SComboButton)
				.ToolTipText(LOCTEXT("ComboButtonTip", "Drag and drop a layer onto this property, or choose one from the drop down."))
				.OnGetMenuContent(this, &FActorLayerPropertyTypeCustomization::OnGetLayerMenu)
				.ButtonStyle(FAppStyle::Get(), "NoBorder")
				.ForegroundColor(FSlateColor::UseForeground())
				.ContentPadding(FMargin(0))
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &FActorLayerPropertyTypeCustomization::GetLayerText)
				]
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(1.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ToolTipText(LOCTEXT("SelectTip", "Select all actors in this layer"))
				.OnClicked(this, &FActorLayerPropertyTypeCustomization::OnSelectLayer)
				.Visibility(this, &FActorLayerPropertyTypeCustomization::GetSelectLayerVisibility)
				.ForegroundColor(FSlateColor::UseForeground())
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
					.Text(FEditorFontGlyphs::Sign_In)
				]
			]
		]
	];
}

FText GetLayerDescription(ULayer* InLayer)
{
	check(InLayer);

	int32 TotalNumActors = Algo::Accumulate(InLayer->GetActorStats(), 0, [](int32 Total, const FLayerActorStats& InStats){ return Total + InStats.Total; });
	return FText::Format(LOCTEXT("LayerNameFormat", "{0} ({1} {1}|plural(one=Actor, other=Actors))"), FText::FromName(InLayer->GetLayerName()), TotalNumActors);
}

FText FActorLayerPropertyTypeCustomization::GetLayerText() const
{
	FName LayerName;
	if (PropertyHandle->GetValue(LayerName) == FPropertyAccess::Success)
	{
		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		ULayer* LayerImpl = Layers->GetLayer(LayerName);
		if (LayerImpl)
		{
			return GetLayerDescription(LayerImpl);
		}

		FText LayerNameText = FText::FromName(LayerName);
		if (LayerName == NAME_None)
		{
			return LayerNameText;
		}
		return FText::Format(LOCTEXT("InvalidLayerNameFormat", "<Invalid> ({0})"), LayerNameText);
	}

	return LOCTEXT("InvalidLayerName", "<Invalid>");
}

TSharedRef<SWidget> FActorLayerPropertyTypeCustomization::OnGetLayerMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	FName LayerName;
	if (PropertyHandle->GetValue(LayerName) == FPropertyAccess::Success && LayerName != NAME_None)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ClearText", "Clear"),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FActorLayerPropertyTypeCustomization::AssignLayer, FName())
			)
		);
		MenuBuilder.AddMenuSeparator();
	}

	MenuBuilder.AddMenuEntry(
		LOCTEXT("OpenLayersBrowser", "Browse Layers..."),
		FText(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Layers"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FActorLayerPropertyTypeCustomization::OpenLayerBrowser)
		)
	);

	MenuBuilder.BeginSection(FName(), LOCTEXT("ExistingLayers", "Existing Layers"));
	{
		TArray<TWeakObjectPtr<ULayer>> AllLayers;
		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		Layers->AddAllLayersTo(AllLayers);

		for (TWeakObjectPtr<ULayer> WeakLayer : AllLayers)
		{
			ULayer* Layer = WeakLayer.Get();
			if (Layer)
			{
				MenuBuilder.AddMenuEntry(
					GetLayerDescription(Layer),
					FText(),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Layer.Icon16x"),
					FUIAction(
						FExecuteAction::CreateSP(this, &FActorLayerPropertyTypeCustomization::AssignLayer, Layer->GetLayerName())
					)
				);
			}
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

EVisibility FActorLayerPropertyTypeCustomization::GetSelectLayerVisibility() const
{
	FName LayerName;
	bool bIsVisible = PropertyHandle->GetValue(LayerName) == FPropertyAccess::Success && !LayerName.IsNone();
	return bIsVisible ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply FActorLayerPropertyTypeCustomization::OnSelectLayer()
{
	FName LayerName;
	if (PropertyHandle->GetValue(LayerName) == FPropertyAccess::Success)
	{
		GEditor->SelectNone(true, true);
		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		Layers->SelectActorsInLayer(LayerName, true, true);
	}
	return FReply::Handled();
}

void FActorLayerPropertyTypeCustomization::AssignLayer(FName InNewLayer)
{
	PropertyHandle->SetValue(InNewLayer);
}

void FActorLayerPropertyTypeCustomization::OpenLayerBrowser()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.GetLevelEditorTabManager()->TryInvokeTab(FTabId("LevelEditorLayerBrowser"));
}

FReply FActorLayerPropertyTypeCustomization::OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<FLayersDragDropOp> LayersDragDropOp = InDragDropEvent.GetOperationAs<FLayersDragDropOp>();
	if (LayersDragDropOp)
	{
		const TArray<FName>& LayerNames = LayersDragDropOp->Layers;
		if (ensure(LayerNames.Num() == 1))
		{
			AssignLayer(LayerNames[0]);
		}
	}
	return FReply::Handled();
}

bool FActorLayerPropertyTypeCustomization::OnVerifyDrag(TSharedPtr<FDragDropOperation> InDragDrop)
{
	return InDragDrop.IsValid() && InDragDrop->IsOfType<FLayersDragDropOp>() && StaticCastSharedPtr<FLayersDragDropOp>(InDragDrop)->Layers.Num() == 1;
}

#undef LOCTEXT_NAMESPACE
