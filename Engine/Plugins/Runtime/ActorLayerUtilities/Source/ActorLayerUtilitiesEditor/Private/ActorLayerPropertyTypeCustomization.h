// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

struct EVisibility;

class FDragDropEvent;
class FReply;
class SWidget;
class FDragDropOperation;
struct FGeometry;

struct FActorLayerPropertyTypeCustomization : public IPropertyTypeCustomization
{
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}

private:

	FText GetLayerText() const;

	TSharedRef<SWidget> OnGetLayerMenu();

	EVisibility GetSelectLayerVisibility() const;

	FReply OnSelectLayer();

	void AssignLayer(FName InNewLayer);

	void OpenLayerBrowser();

	FReply OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);
	bool OnVerifyDrag(TSharedPtr<FDragDropOperation> InDragDrop);

	TSharedPtr<IPropertyHandle> PropertyHandle;
};

