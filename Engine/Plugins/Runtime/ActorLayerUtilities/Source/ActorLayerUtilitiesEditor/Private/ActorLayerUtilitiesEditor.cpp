// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ActorLayerUtilities.h"
#include "ActorLayerPropertyTypeCustomization.h"

LLM_DEFINE_TAG(ActionLayerUtilities_ActorLayerUtilitiesEditor);

class FActorLayerUtilitiesEditorModule : public IModuleInterface
{
	FName ActorLayerTypeName;

	static TSharedRef<IPropertyTypeCustomization> MakeCustomization()
	{
		return MakeShared<FActorLayerPropertyTypeCustomization>();
	}

	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(ActionLayerUtilities_ActorLayerUtilitiesEditor);

		ActorLayerTypeName = FActorLayer::StaticStruct()->GetFName();

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomPropertyTypeLayout(ActorLayerTypeName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(MakeCustomization));
	}

	virtual void ShutdownModule() override
	{
		LLM_SCOPE_BYTAG(ActionLayerUtilities_ActorLayerUtilitiesEditor);

		FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");

		if (PropertyModule && ActorLayerTypeName != NAME_None)
		{
			PropertyModule->UnregisterCustomPropertyTypeLayout(ActorLayerTypeName);
		}
	}

};


IMPLEMENT_MODULE(FActorLayerUtilitiesEditorModule, ActorLayerUtilitiesEditor)